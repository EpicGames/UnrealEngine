// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class NewLevelDialog : ModuleRules
{
	public NewLevelDialog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"ToolWidgets",
			}
		);
	}
}
