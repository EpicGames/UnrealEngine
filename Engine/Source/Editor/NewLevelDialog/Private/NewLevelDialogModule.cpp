// Copyright Epic Games, Inc. All Rights Reserved.

#include "NewLevelDialogModule.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STileView.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Texture2D.h"
#include "UnrealEdGlobals.h"
#include "Internationalization/BreakIterator.h"
#include "Brushes/SlateImageBrush.h"
#include "SPrimaryButton.h"
#include "Engine/Level.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "NewLevelDialog"

namespace NewLevelDialogDefs
{
	constexpr float TemplateTileHeight = 153;
	constexpr float TemplateTileWidth = 102;
	constexpr float DefaultWindowHeight = 418;
	constexpr float DefaultWindowWidth = 527;
	constexpr float LargeWindowHeight = 566;
	constexpr float LargeWindowWidth = 1008;
	constexpr float MinWindowHeight = 280;
	constexpr float MinWindowWidth = 320;
}

struct FNewLevelTemplateItem
{
	FTemplateMapInfo TemplateMapInfo;
	FText Name;
	FString Category;
	TUniquePtr<FSlateBrush> ThumbnailBrush;
	TObjectPtr<UTexture2D> ThumbnailAsset = nullptr;
	int32 OriginalIndex = INDEX_NONE;

	enum NewLevelType
	{
		Empty,
		EmptyWorldPartition,
		Template
	} Type;
};

/**
 * Single thumbnail tile for a level template in the tile view of templates.
 */
class SNewLevelTemplateTile : public STableRow<TSharedPtr<FNewLevelTemplateItem>>
{
public:
	SLATE_BEGIN_ARGS(SNewLevelTemplateTile) {}
		SLATE_ARGUMENT(TSharedPtr<FNewLevelTemplateItem>, Item)
	SLATE_END_ARGS()

	static TSharedRef<ITableRow> BuildTile(TSharedPtr<FNewLevelTemplateItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		if (!ensure(Item.IsValid()))
		{
			return SNew(STableRow<TSharedPtr<FNewLevelTemplateItem>>, OwnerTable);
		}

		return SNew(SNewLevelTemplateTile, OwnerTable).Item(Item);
	}

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
	{
		check(InArgs._Item.IsValid());

		STableRow::Construct(
			STableRow::FArguments()
			.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
			.Padding(2.0f)
			.Content()
			[
				SNew(SBorder)
				.Padding(FMargin(0.0f, 0.0f, 5.0f, 5.0f))
				.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SVerticalBox)
						// Thumbnail
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(NewLevelDialogDefs::TemplateTileWidth)
							.HeightOverride(NewLevelDialogDefs::TemplateTileWidth) // use width on purpose, this is a square
							[
								SNew(SBorder)
								.Padding(FMargin(0))
								.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.ThumbnailAreaBackground"))
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								[
									SNew(SImage)
									.Image(InArgs._Item->ThumbnailBrush.Get())
								]
							]
						]
						// Name
						+ SVerticalBox::Slot()
						[
							SNew(SBorder)
							.Padding(FMargin(5.0f, 0))
							.VAlign(VAlign_Top)
							.Padding(FMargin(3.0f, 3.0f))
							.BorderImage(this, &SNewLevelTemplateTile::GetNameAreaBackgroundBrush)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
								.WrapTextAt(NewLevelDialogDefs::TemplateTileWidth - 4.0f)
								.LineBreakPolicy(FBreakIterator::CreateCamelCaseBreakIterator())
								.Text(InArgs._Item->Name)
								.ColorAndOpacity(this, &SNewLevelTemplateTile::GetNameAreaTextColor)
							]
						]
					]
					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Visibility(EVisibility::HitTestInvisible)
						.Image(this, &SNewLevelTemplateTile::GetSelectionOutlineBrush)
					]
				]
			],
			OwnerTable
		);
	}

private:
	const FSlateBrush* GetSelectionOutlineBrush() const
	{
		const bool bIsSelected = IsSelected();
		const bool bIsTileHovered = IsHovered();

		if (bIsSelected && bIsTileHovered)
		{
			static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
			return FAppStyle::Get().GetBrush(SelectedHover);
		}
		else if (bIsSelected)
		{
			static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
			return FAppStyle::Get().GetBrush(Selected);
		}
		else if (bIsTileHovered)
		{
			static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
			return FAppStyle::Get().GetBrush(Hovered);
		}

		return FStyleDefaults::GetNoBrush();
	}

	const FSlateBrush* GetNameAreaBackgroundBrush() const
	{
		const bool bIsSelected = IsSelected();
		const bool bIsRowHovered = IsHovered();

		if (bIsSelected && bIsRowHovered)
		{
			static const FName SelectedHover("ProjectBrowser.ProjectTile.NameAreaSelectedHoverBackground");
			return FAppStyle::Get().GetBrush(SelectedHover);
		}
		else if (bIsSelected)
		{
			static const FName Selected("ProjectBrowser.ProjectTile.NameAreaSelectedBackground");
			return FAppStyle::Get().GetBrush(Selected);
		}
		else if (bIsRowHovered)
		{
			static const FName Hovered("ProjectBrowser.ProjectTile.NameAreaHoverBackground");
			return FAppStyle::Get().GetBrush(Hovered);
		}

		return FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.NameAreaBackground");
	}

	FSlateColor GetNameAreaTextColor() const
	{
		const bool bIsSelected = IsSelected();
		const bool bIsRowHovered = IsHovered();

		if (bIsSelected || bIsRowHovered)
		{
			return FStyleColors::White;
		}

		return FSlateColor::UseForeground();
	}
};

/**
 * Main widget class showing a table of level templates as labeled thumbnails
 * for the user to select by clicking.
 */
class SNewLevelDialog : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SNewLevelDialog) {}
		/** A pointer to the parent window */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)

		SLATE_ATTRIBUTE(TArray<FTemplateMapInfo>, Templates)
		SLATE_ATTRIBUTE(bool, bShowPartitionedTemplates)
		SLATE_ATTRIBUTE(FNewLevelDialogModule::EShowEmptyTemplate, ShowEmptyTemplate)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ParentWindowPtr = InArgs._ParentWindow.Get();

		TemplateItemsList = MakeTemplateItems(InArgs._bShowPartitionedTemplates.Get(), InArgs._ShowEmptyTemplate.Get(), InArgs._Templates.Get());
		
		TemplateListView = SNew(STileView<TSharedPtr<FNewLevelTemplateItem>>)
			.ListItemsSource(&TemplateItemsList)
			.SelectionMode(ESelectionMode::Single)
			.ClearSelectionOnClick(false)
			.ItemAlignment(EListItemAlignment::LeftAligned)
			.OnGenerateTile_Static(&SNewLevelTemplateTile::BuildTile)
			.OnMouseButtonDoubleClick(this, &SNewLevelDialog::HandleTemplateItemDoubleClick)
			.ItemHeight(NewLevelDialogDefs::TemplateTileHeight + 9)
			.ItemWidth(NewLevelDialogDefs::TemplateTileWidth + 9);

		TSharedPtr<SButton> CreateButton;

		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(8.0f, 8.0f))
			[
				SNew(SVerticalBox)
				// Top section with template thumbnails
				+SVerticalBox::Slot()
				[
					SNew(SScrollBorder, TemplateListView.ToSharedRef())
					[
						TemplateListView.ToSharedRef()
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(-8.0f, 0.0f)
				[
					SNew(SSeparator)
					.Orientation(EOrientation::Orient_Horizontal)
					.Thickness(2.0f)
				]
				// Bottom section with dialog buttons
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(8.0f, 16.0f, 8.0f, 8.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(8.0f, 0.0f, 8.0f, 0.0f)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SAssignNew(CreateButton, SPrimaryButton)
						.Text(LOCTEXT("Create", "Create"))
						.IsEnabled(this, &SNewLevelDialog::CanCreateLevel)
						.OnClicked(this, &SNewLevelDialog::OnCreateClicked)
					]
					+ SHorizontalBox::Slot()
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked(this, &SNewLevelDialog::OnCancelClicked)
					]
				]
			]
		];

		// Give the create button inital focus so that pressing enter will activate it
		check(CreateButton != nullptr);
		ParentWindowPtr.Pin().Get()->SetWidgetToFocusOnActivate( CreateButton );

		// Automatically select the first template item by default
		if (!TemplateItemsList.IsEmpty())
		{
			TemplateListView->SetSelection(TemplateItemsList[0]);
		}
	}

	FString GetChosenTemplate() const
	{
		FString Result;
		if (OutTemplateItem.IsValid())
		{
			Result = OutTemplateItem->TemplateMapInfo.Map.GetLongPackageName();
			// For backwards compatibility, handle the case where the map name format does not include the object name directly
			if (Result.IsEmpty())
			{
				Result = OutTemplateItem->TemplateMapInfo.Map.GetAssetPathString();
			}
		}

		return Result;
	}

	FTemplateMapInfo GetChosenTemplateMapInfo() const { return OutTemplateItem.IsValid() ? OutTemplateItem->TemplateMapInfo : FTemplateMapInfo(); }
	bool IsPartitionedWorld() const { return OutTemplateItem.IsValid() && OutTemplateItem->Type == FNewLevelTemplateItem::NewLevelType::EmptyWorldPartition; }
	bool IsTemplateChosen() const { return OutTemplateItem.IsValid(); }

	//~ Begin FGCObject Interface.
	virtual void AddReferencedObjects(FReferenceCollector& Collector)  override
	{
		for (const TSharedPtr<FNewLevelTemplateItem>& It : TemplateItemsList)
		{
			if (It.IsValid() && It->ThumbnailAsset)
			{
				Collector.AddReferencedObject(It->ThumbnailAsset);
			}
		}
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("SNewLevelDialog");
	}
	//~ End FGCObject Interface.

private:
	static TArray<TSharedPtr<FNewLevelTemplateItem>> MakeTemplateItems(bool bShowPartitionedTemplates, FNewLevelDialogModule::EShowEmptyTemplate ShowEmptyTemplate, const TArray<FTemplateMapInfo>& TemplateMapInfos)
	{
		TArray<TSharedPtr<FNewLevelTemplateItem>> TemplateItems;

		// Build a list of items - one for each template
		int32 CurrentIndex = 0;
		for (const FTemplateMapInfo& TemplateMapInfo : TemplateMapInfos)
		{
			if (!bShowPartitionedTemplates && ULevel::GetIsLevelPartitionedFromPackage(FName(*TemplateMapInfo.Map.ToString())))
			{
				CurrentIndex++;
				continue;
			}

			TSharedPtr<FNewLevelTemplateItem> Item = MakeShareable(new FNewLevelTemplateItem());
			Item->TemplateMapInfo = TemplateMapInfo;
			Item->Type = FNewLevelTemplateItem::NewLevelType::Template;
			Item->Name = TemplateMapInfo.DisplayName;
			Item->Category = TemplateMapInfo.Category;
			Item->OriginalIndex = CurrentIndex++;

			TSoftObjectPtr<UTexture2D> ThumbnailSoftObjectPtr;
			if (TemplateMapInfo.Thumbnail.IsValid())
			{
				ThumbnailSoftObjectPtr = TemplateMapInfo.Thumbnail;
			}
			else if (!TemplateMapInfo.ThumbnailTexture.IsNull())
			{
				ThumbnailSoftObjectPtr = TemplateMapInfo.ThumbnailTexture;
			}

			UTexture2D* ThumbnailTexture = nullptr;
			if (!ThumbnailSoftObjectPtr.IsNull())
			{
				ThumbnailTexture = ThumbnailSoftObjectPtr.Get();
				if (!ThumbnailTexture)
				{
					ThumbnailTexture = ThumbnailSoftObjectPtr.LoadSynchronous();
					if (ThumbnailTexture)
					{
						// Avoid calling UpdateResource on cooked texture as doing so will destroy the texture's data
						if (!ThumbnailTexture->GetPackage()->HasAllPackagesFlags(PKG_FilterEditorOnly))
						{
							// Newly loaded texture requires async work to complete before being rendered in modal dialog
							ThumbnailTexture->FinishCachePlatformData();
							ThumbnailTexture->UpdateResource();
						}
					}
				}
			}

			if (ThumbnailTexture)
			{
				// Reference to prevent garbage collection
				Item->ThumbnailAsset = ThumbnailTexture;

				// Level with thumbnail
				Item->ThumbnailBrush = MakeUnique<FSlateImageBrush>(
					ThumbnailTexture,
					FVector2D(ThumbnailTexture->GetSizeX(), ThumbnailTexture->GetSizeY()));

				if (Item->Name.IsEmpty())
				{
					Item->Name = FText::FromString(ThumbnailTexture->GetName().Replace(TEXT("_"), TEXT(" ")));
				}
			}
			else
			{
				// Level with no thumbnail
				Item->ThumbnailBrush = MakeUnique<FSlateBrush>(*FAppStyle::Get().GetBrush("UnrealDefaultThumbnail"));

				if (Item->Name.IsEmpty())
				{
					Item->Name = FText::FromString(FPaths::GetBaseFilename(TemplateMapInfo.Map.ToString()));
				}
			}

			check(Item->ThumbnailBrush);
			Item->ThumbnailBrush->OutlineSettings.CornerRadii = FVector4(4, 4, 0, 0);
			Item->ThumbnailBrush->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
			Item->ThumbnailBrush->DrawAs = ESlateBrushDrawType::RoundedBox;

			TemplateItems.Add(Item);
		}

		// Add an extra item for creating a new, blank level
		if (ShowEmptyTemplate == FNewLevelDialogModule::EShowEmptyTemplate::Show)
		{
			TSharedPtr<FNewLevelTemplateItem> NewItem = MakeShareable(new FNewLevelTemplateItem());
			NewItem->Type = FNewLevelTemplateItem::NewLevelType::Empty;
			NewItem->Name = LOCTEXT("NewLevelItemLabel", "Empty Level");
			NewItem->ThumbnailBrush = MakeUnique<FSlateBrush>(*FAppStyle::GetBrush("NewLevelDialog.Blank"));
			NewItem->ThumbnailBrush->OutlineSettings.CornerRadii = FVector4(4, 4, 0, 0);
			NewItem->ThumbnailBrush->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
			NewItem->ThumbnailBrush->DrawAs = ESlateBrushDrawType::RoundedBox;
			TemplateItems.Add(NewItem);
		}

		const FString& OpenWorldCategory = TEXT("OpenWorld");

		if (bShowPartitionedTemplates)
		{
			// Add an extra item for creating a new, blank level
			TSharedPtr<FNewLevelTemplateItem> NewItemWP = MakeShareable(new FNewLevelTemplateItem());
			NewItemWP->Type = FNewLevelTemplateItem::NewLevelType::EmptyWorldPartition;
			NewItemWP->Name = LOCTEXT("NewWPLevelItemLabel", "Empty Open World");
			NewItemWP->Category = OpenWorldCategory;
			NewItemWP->ThumbnailBrush = MakeUnique<FSlateBrush>(*FAppStyle::GetBrush("NewLevelDialog.BlankWP"));
			NewItemWP->ThumbnailBrush->OutlineSettings.CornerRadii = FVector4(4, 4, 0, 0);
			NewItemWP->ThumbnailBrush->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
			NewItemWP->ThumbnailBrush->DrawAs = ESlateBrushDrawType::RoundedBox;
			TemplateItems.Add(NewItemWP);
		}

		// OpenWorld category always first, If Categories are the same compares Type which will put Templates before Empty maps
		TemplateItems.Sort([&OpenWorldCategory](const TSharedPtr<FNewLevelTemplateItem>& LHS, const TSharedPtr<FNewLevelTemplateItem>& RHS)
		{
			if (LHS->Category == RHS->Category)
			{
				if (LHS->Type == RHS->Type)
				{
					return LHS->OriginalIndex < RHS->OriginalIndex;
				}

				return LHS->Type > RHS->Type;
			}
			else if (LHS->Category == OpenWorldCategory)
			{
				return true;
			}
			else if (RHS->Category == OpenWorldCategory)
			{
				return false;
			}

			return LHS->Category > RHS->Category;
		});

		return TemplateItems;
	}

	bool CanCreateLevel() const
	{
		if (!ensure(TemplateListView.IsValid()))
		{
			return false;
		}

		return TemplateListView->GetNumItemsSelected() == 1;
	}

	FReply OnCreateClicked()
	{
		if (!ensure(TemplateListView.IsValid()))
		{
			return FReply::Handled();
		}

		const TArray<TSharedPtr<FNewLevelTemplateItem>> Items = TemplateListView->GetSelectedItems();
		if (!ensure(Items.Num() == 1))
		{
			return FReply::Handled();
		}

		OutTemplateItem = Items[0];

		ParentWindowPtr.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnCancelClicked()
	{
		ParentWindowPtr.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCancelClicked();
		}

		return SCompoundWidget::OnKeyDown( MyGeometry, InKeyEvent );
	}

	void HandleTemplateItemDoubleClick(TSharedPtr<FNewLevelTemplateItem>)
	{
		OnCreateClicked();
	}

	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** Initial size of the parent window */
	FVector2D InitialWindowSize;

	TArray<TSharedPtr<FNewLevelTemplateItem>> TemplateItemsList;
	TSharedPtr < STileView<TSharedPtr<FNewLevelTemplateItem>> > TemplateListView;
	TSharedPtr<FNewLevelTemplateItem> OutTemplateItem;
};

IMPLEMENT_MODULE( FNewLevelDialogModule, NewLevelDialog );

const FName FNewLevelDialogModule::NewLevelDialogAppIdentifier( TEXT( "NewLevelDialogApp" ) );

void FNewLevelDialogModule::StartupModule()
{
}

void FNewLevelDialogModule::ShutdownModule()
{
}

bool FNewLevelDialogModule::CreateAndShowNewLevelDialog( const TSharedPtr<const SWidget> ParentWidget, FString& OutTemplateMapPackageName, bool bShowPartitionedTemplates, bool& bOutIsPartitionedWorld, FTemplateMapInfo* OutTemplateMapInfo, EShowEmptyTemplate ShowEmptyTemplate)
{
	TArray<FTemplateMapInfo> EmptyTemplates;
	return CreateAndShowTemplateDialog(ParentWidget, LOCTEXT("WindowHeader", "New Level"), GUnrealEd ? GUnrealEd->GetTemplateMapInfos() : EmptyTemplates, OutTemplateMapPackageName, bShowPartitionedTemplates, bOutIsPartitionedWorld, OutTemplateMapInfo, ShowEmptyTemplate);
}

bool FNewLevelDialogModule::CreateAndShowTemplateDialog( const TSharedPtr<const SWidget> ParentWidget, const FText& Title, const TArray<FTemplateMapInfo>& Templates, FString& OutTemplateMapPackageName, bool bShowPartitionedTemplates, bool& bOutIsPartitionedWorld, FTemplateMapInfo* OutTemplateMapInfo, EShowEmptyTemplate ShowEmptyTemplate)
{
	// Open larger window if there are enough templates
	FVector2D WindowClientSize(NewLevelDialogDefs::DefaultWindowWidth, NewLevelDialogDefs::DefaultWindowHeight);
	if (Templates.Num() > 9)
	{
		WindowClientSize = FVector2D(NewLevelDialogDefs::LargeWindowWidth, NewLevelDialogDefs::LargeWindowHeight);
	}

	TSharedPtr<SWindow> NewLevelWindow =
		SNew(SWindow)
		.Title(Title)
		.ClientSize(WindowClientSize)
		.MinHeight(NewLevelDialogDefs::MinWindowHeight)
		.MinWidth(NewLevelDialogDefs::MinWindowWidth)
		.SizingRule( ESizingRule::UserSized )
		.SupportsMinimize(false)
		.SupportsMaximize(false);

	TSharedRef<SNewLevelDialog> NewLevelDialog =
		SNew(SNewLevelDialog)
		.ParentWindow(NewLevelWindow)
		.Templates(Templates)
		.bShowPartitionedTemplates(bShowPartitionedTemplates)
		.ShowEmptyTemplate(ShowEmptyTemplate);

	NewLevelWindow->SetContent(NewLevelDialog);

	FSlateApplication::Get().AddModalWindow(NewLevelWindow.ToSharedRef(), ParentWidget);

	if (OutTemplateMapInfo)
	{
		*OutTemplateMapInfo = NewLevelDialog->GetChosenTemplateMapInfo();
	}

	OutTemplateMapPackageName = NewLevelDialog->GetChosenTemplate();
	bOutIsPartitionedWorld = NewLevelDialog->IsPartitionedWorld();
	
	return NewLevelDialog->IsTemplateChosen();
}

#undef LOCTEXT_NAMESPACE
