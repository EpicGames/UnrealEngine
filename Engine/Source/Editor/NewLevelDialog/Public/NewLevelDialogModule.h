// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"
#include "Editor/UnrealEdEngine.h"

struct FTemplateMapInfo;

/**
 * New Level Dialog module
 */
class FNewLevelDialogModule : public IModuleInterface
{
public:

	/**
	 * Control if 'Empty Level' template should appear or not
	 */
	enum class EShowEmptyTemplate : uint8
	{
		Show,
		Hide
	};

	/**
	 * Called right after the plugin DLL has been loaded and the plugin object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the plugin is unloaded, right before the plugin object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Creates and show a window with an SNewLevelDialog
	 * 
	 * @param	ParentWidget - The parent widget for the modal window showing the dialog
	 * @param	OutTemplateName	- (out) The package name of the template map selected by the user. Empty if blank map selected.
	 * @param	bShowPartitionedTemplates - If true, we allow partitioned world templates to be shown in the dialog.
	 * @param	bOutPartitionedWorld - (out) If true and OutTemplateName is empty, the empty map should be partitioned.
	 * @param	OutTemplateMapInfo - (out) If not null, will be set to the template map selected by the user.
	 * @param	ShowEmptyTemplate - Control if 'Empty Level' template should appear or not
	 * @return	true if the user selected a valid item, false if the user canceled
	 */
	virtual bool CreateAndShowNewLevelDialog( const TSharedPtr<const SWidget> ParentWidget, FString& OutTemplateMapPackageName, bool bShowPartitionedTemplates, bool& bOutPartitionedWorld, FTemplateMapInfo* OutTemplateMapInfo = nullptr, EShowEmptyTemplate ShowEmptyTemplate = EShowEmptyTemplate::Show);

	/**
	 * Creates and show a window with an SNewLevelDialog
	 *
	 * @param	ParentWidget - The parent widget for the modal window showing the dialog
	 * @param   Title - The dialog's title
	 * @param   Templates - The list of template to be shown in the dialog
	 * @param	OutTemplateName	- (out) The package name of the template map selected by the user. Empty if blank map selected.
	 * @param	bShowPartitionedTemplates - If true, we allow partitioned world templates to be shown in the dialog.
	 * @param	bOutPartitionedWorld - (out) If true and OutTemplateName is empty, the empty map should be partitioned.
	 * @param	OutTemplateMapInfo - (out) If not null, will be set to the template map selected by the user.
	 * @param	ShowEmptyTemplate - Control if 'Empty Level' template should appear or not
	 * @return	true if the user selected a valid item, false if the user canceled
	 */
	virtual bool CreateAndShowTemplateDialog(const TSharedPtr<const SWidget> ParentWidget, const FText& Title, const TArray<FTemplateMapInfo>& Templates, FString& OutTemplateMapPackageName, bool bShowPartitionedTemplates, bool& bOutPartitionedWorld, FTemplateMapInfo* OutTemplateMapInfo = nullptr, EShowEmptyTemplate ShowEmptyTemplate = EShowEmptyTemplate::Show);

	/** New Level Dialog app identifier string */
	static const FName NewLevelDialogAppIdentifier;

};
